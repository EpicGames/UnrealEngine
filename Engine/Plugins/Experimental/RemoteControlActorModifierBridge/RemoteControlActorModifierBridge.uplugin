{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Remote Control Actor Modifier Bridge",
	"Description": "Interface between the Remote Control, Actor Modifier and Property Animator plugins.",
	"Category": "Messaging",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "ActorModifierCore",
			"Enabled": true
		},
		{
			"Name": "OperatorStack",
			"Enabled": true
		},
		{
			"Name": "RemoteControl",
			"Enabled": true
		},
		{
			"Name": "PropertyAnimatorCore",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "RemoteControlActorModifierBridge",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}