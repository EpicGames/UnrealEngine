// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlActorModifierBridgeModule.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Components/ActorComponent.h"
#include "Containers/Array.h"
#include "GameFramework/Actor.h"
#include "IRemoteControlUIModule.h"
#include "Modifiers/ActorModifierCoreBase.h"
#include "PropertyPath.h"
#include "RemoteControlField.h"
#include "Subsystems/OperatorStackEditorSubsystem.h"
#include "UObject/NameTypes.h"

namespace UE::FRemoteControlActorModifierBridge::Private
{
	const FLazyName ResolverName = "ActorModifierResolver";
}

void FRemoteControlActorModifierBridgeModule::StartupModule()
{
	using namespace UE::FRemoteControlActorModifierBridge::Private;

	IRemoteControlUIModule::Get().RegisterPropertyResolver(
		ResolverName,
		FRCPropertyResolver::CreateRaw(this, &FRemoteControlActorModifierBridgeModule::ResolverActorModifierProperty)
	);
}

void FRemoteControlActorModifierBridgeModule::ShutdownModule()
{
	using namespace UE::FRemoteControlActorModifierBridge::Private;

	IRemoteControlUIModule::Get().UnregisterPropertyResolver(ResolverName);
}

bool FRemoteControlActorModifierBridgeModule::ResolverActorModifierProperty(const TSharedRef<FRemoteControlProperty>& InProperty, TArray<UObject*>& InOutBoundObjects, TSharedPtr<FPropertyPath>& OutPropertyPath)
{
	const UActorModifierCoreBase* FoundModifier = nullptr;
	const UPropertyAnimatorCoreBase* FoundAnimator = nullptr;

	for (auto It = InOutBoundObjects.CreateIterator(); It; ++It)
	{
		const UObject* BindingObject = *It;

		if (const UActorModifierCoreBase* Modifier = Cast<UActorModifierCoreBase>(BindingObject))
		{
			It.RemoveCurrent();

			if (AActor* Actor = Modifier->GetModifiedActor())
			{
				InOutBoundObjects.Add(Actor);
			}

			if (!FoundModifier)
			{
				FoundModifier = Modifier;
			}
		}
		else if (const UPropertyAnimatorCoreBase* Animator = Cast<UPropertyAnimatorCoreBase>(BindingObject))
		{
			It.RemoveCurrent();

			if (AActor* Actor = Animator->GetAnimatorActor())
			{
				InOutBoundObjects.Add(Actor);
			}

			if (!FoundAnimator)
			{
				FoundAnimator = Animator;
			}

			if (!OutPropertyPath.IsValid())
			{
				for (const FPropertyAnimatorCoreData& CoreData : Animator->GetLinkedProperties())
				{
					if (FProperty* MemberProperty = CoreData.GetMemberProperty())
					{
						if (MemberProperty->HasAnyPropertyFlags(CPF_Edit | CPF_EditConst))
						{
							OutPropertyPath = FPropertyPath::Create(MemberProperty);
							break;
						}
					}
				}
			}
		}
	}

	if (FoundModifier)
	{
		UOperatorStackEditorSubsystem::Get()->FocusCustomizationWidget(FoundModifier, TEXT("Modifiers"));
		return true;
	}

	if (FoundAnimator)
	{
		UOperatorStackEditorSubsystem::Get()->FocusCustomizationWidget(FoundAnimator, TEXT("Animators"));
		return true;
	}

	return false;
}

IMPLEMENT_MODULE(FRemoteControlActorModifierBridgeModule, RemoteControlActorModifierBridge);
