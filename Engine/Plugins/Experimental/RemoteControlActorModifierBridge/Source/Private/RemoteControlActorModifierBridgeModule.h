// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "Containers/ContainersFwd.h"
#include "Templates/SharedPointerFwd.h"

class FPropertyPath;
class UObject;
struct FRemoteControlProperty;

class FRemoteControlActorModifierBridgeModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

private:
	bool ResolverActorModifierProperty(const TSharedRef<FRemoteControlProperty>& InProperty, TArray<UObject*>& InOutBoundObjects, TSharedPtr<FPropertyPath>& OutPropertyPath);
};
