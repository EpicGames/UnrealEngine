// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RemoteControlActorModifierBridge : ModuleRules
{
	public RemoteControlActorModifierBridge(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ActorModifierCore",
				"Core",
				"CoreUObject",
				"OperatorStackEditor",
				"PropertyEditor",
				"PropertyAnimatorCore",
				"RemoteControl",
				"RemoteControlUI"
			}
		);
	}
}